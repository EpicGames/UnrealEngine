// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CollisionAnalyzer : ModuleRules
{
	public CollisionAnalyzer(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateIncludePathModuleNames.AddRange(
            new string[] {
				"DesktopPlatform",
			}
        );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "InputCore",
				"Slate",
				"SlateCore",
                
				"Engine",
				"WorkspaceMenuStructure",
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
				"DesktopPlatform",
				"MainFrame",
			}
        );
	}
}
