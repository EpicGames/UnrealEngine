// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollisionAnalyzer.h"
#include "HAL/FileManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "Engine/HitResult.h"
#include "DrawDebugHelpers.h"
#include "SCollisionAnalyzer.h"
#include "CollisionAnalyzerLog.h"
#include "CollisionDebugDrawingPublic.h"


/** Magic value, determining that file is a collision analyzer file.				*/
#define COLLISION_ANALYZER_MAGIC					0x2DFF34FC
/** Version of collision analyzer. Incremented on serialization changes.			*/
#define COLLISION_ANALYZER_VERSION					0

FCAQuery::FCAQuery() :
	Params(NAME_None, FCollisionQueryParams::GetUnknownStatId())
{
}
FCAQuery::~FCAQuery() = default;

/** Util for serializing an FHitResult struct */
void SerializeHitResult(FArchive& Ar, FHitResult& Result)
{
	bool bTempBlocking = Result.bBlockingHit;
	Ar << bTempBlocking;
	Result.bBlockingHit = bTempBlocking;

	bool bTempPenetrating = Result.bStartPenetrating;
	Ar << bTempPenetrating;
	Result.bStartPenetrating = bTempPenetrating;

	Ar << Result.Time;
	Ar << Result.Distance;
	Ar << Result.Location;
	Ar << Result.ImpactPoint;
	Ar << Result.Normal;
	Ar << Result.ImpactNormal;
	Ar << Result.TraceStart;
	Ar << Result.TraceEnd;
	Ar << Result.PenetrationDepth;
	Ar << Result.BoneName;
	Ar << Result.PhysMaterial;
	Ar << Result.HitObjectHandle;
	Ar << Result.Component;
	Ar << Result.FaceIndex;
}

FArchive& operator << (FArchive& Ar, FCAQuery& Query)
{
	Ar << Query.Start;
	Ar << Query.End;
	Ar << Query.Rot;
	Ar << (int32&)Query.Type;
	Ar << (int32&)Query.Shape;
	Ar << (int32&)Query.Mode;
	Ar << Query.Dims;
	Ar << (int32&)Query.Channel;

	Ar << Query.Params.TraceTag;
	Ar << Query.Params.OwnerTag;

	bool bTraceAsyncDeprecated = false;
	Ar << bTraceAsyncDeprecated;
	Ar << Query.Params.bTraceComplex;
	Ar << Query.Params.bFindInitialOverlaps;
	Ar << Query.Params.bReturnFaceIndex;
	Ar << Query.Params.bReturnPhysicalMaterial;

	int32 NumResults = 0;

	if (Ar.IsLoading())
	{
		// Load array.
		Ar << NumResults;
		Query.Results.SetNum(NumResults);
	}
	else if(Ar.IsSaving())
	{
		// Save array.
		NumResults = Query.Results.Num();
		Ar << NumResults;
	}

	for (int32 i = 0; i < NumResults; i++)
	{
		SerializeHitResult(Ar, Query.Results[i]);
	}

	Ar << Query.FrameNum;
	Ar << Query.CPUTime;
	Ar << Query.ID;

	return Ar;
}


//////////////////////////////////////////////////////////////////////////


void FCollisionAnalyzer::CaptureQuery(	const FVector& Start, 
										const FVector& End, 
										const FQuat& Rot, 
										ECAQueryType::Type QueryType, 
										ECAQueryShape::Type QueryShape,
										ECAQueryMode::Type QueryMode,
										const FVector& Dims, 
										ECollisionChannel TraceChannel, 
										const struct FCollisionQueryParams& Params, 
										const FCollisionResponseParams&	ResponseParams,
										const FCollisionObjectQueryParams&	ObjectParams,
										const TArray<FHitResult>& Results, 
										const TArray<FHitResult>& TouchAllResults,
										double CPUTime) 
{
	if(bIsRecording)
	{
		int32 NewQueryId = Queries.AddZeroed();
		FCAQuery& NewQuery = Queries[NewQueryId];
		NewQuery.Start = Start;
		NewQuery.End = End;
		NewQuery.Rot = Rot;
		NewQuery.Type = QueryType;
		NewQuery.Shape = QueryShape;
		NewQuery.Mode = QueryMode;
		NewQuery.Dims = Dims;
		NewQuery.Channel = TraceChannel;
		NewQuery.Params = Params;
		NewQuery.ResponseParams = ResponseParams;
		NewQuery.ObjectParams = ObjectParams;
		NewQuery.Results = Results;
		NewQuery.TouchAllResults = TouchAllResults;
		NewQuery.FrameNum = CurrentFrameNum;
		NewQuery.CPUTime = CPUTime * 1000.f;		
		NewQuery.ID = NewQueryId;


		QueryAddedEvent.Broadcast();
	}
}

TSharedPtr<SWidget> FCollisionAnalyzer::SummonUI() 
{
	TSharedPtr<SWidget> ReturnWidget;

	UE_LOGF(LogCollisionAnalyzer, Log, "Opening CollisionAnalyzer...");

	if( IsInGameThread() )
	{
		// Make a window
		ReturnWidget = SNew(SCollisionAnalyzer, this);			
	}
	else
	{
		UE_LOGF(LogCollisionAnalyzer, Warning, "FCollisionAnalyzer::DisplayUI: Not in game thread.");
	}

	return ReturnWidget;
}

bool FCollisionAnalyzer::IsRecording()
{
	return bIsRecording;
}

void FCollisionAnalyzer::TickAnalyzer(UWorld* World)
{
	if(bIsRecording)
	{
		// Increment frame number
		CurrentFrameNum++;
	}

	// Draw any queries requested
	for(int32 DrawIdx=0; DrawIdx<DrawQueryIndices.Num(); DrawIdx++)
	{
		int32 QueryIdx = DrawQueryIndices[DrawIdx];
		if(QueryIdx < Queries.Num())
		{
			FCAQuery& DrawQuery = Queries[QueryIdx];
			if(DrawQuery.Type == ECAQueryType::Raycast)
			{
				DrawLineTraces(World, DrawQuery.Start, DrawQuery.End, DrawQuery.Results, 0.f);
			}
			else if(DrawQuery.Type == ECAQueryType::GeomSweep)
			{
				if (DrawQuery.Shape == ECAQueryShape::Sphere)
				{
					DrawSphereSweeps(World, DrawQuery.Start, DrawQuery.End, DrawQuery.Dims.X, DrawQuery.Results, 0.f);
				}
				else if (DrawQuery.Shape == ECAQueryShape::Box)
				{
					DrawBoxSweeps(World, DrawQuery.Start, DrawQuery.End, DrawQuery.Dims, DrawQuery.Rot, DrawQuery.Results, 0.f);
				}
				else if (DrawQuery.Shape == ECAQueryShape::Capsule)
				{
					DrawCapsuleSweeps(World, DrawQuery.Start, DrawQuery.End, DrawQuery.Dims.Z, DrawQuery.Dims.X, DrawQuery.Rot, DrawQuery.Results, 0.f);
				}
			}
		}
	}

	// Draw debug box if desired
	if(DrawBox.IsValid)
	{
		DrawDebugBox(World, DrawBox.GetCenter(), DrawBox.GetExtent(), FColor::White);
	}
}


void FCollisionAnalyzer::SetIsRecording(bool bNewRecording)
{
	if(bNewRecording != bIsRecording)
	{
		// If starting recording, reset queries and zero frame counter.
		if(bNewRecording)
		{
			Queries.Reset();
			DrawQueryIndices.Empty();
			CurrentFrameNum = 0;

			QueriesChangedEvent.Broadcast();
		}

		bIsRecording = bNewRecording;
	}
}

int32 FCollisionAnalyzer::GetNumFramesOfRecording()
{
	return CurrentFrameNum+1;
}

void FCollisionAnalyzer::SaveCollisionProfileData(FString ProfileFileName)
{
	FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*ProfileFileName);

	if(FileWriter != nullptr)
	{
		FCollisionAnalyzerProxyArchive Ar(*FileWriter);

		int32 Magic = COLLISION_ANALYZER_MAGIC;
		int32 Version = COLLISION_ANALYZER_VERSION;

		Ar << Magic;
		Ar << Version;
		Ar << Queries;

		FileWriter->Close();

		delete FileWriter;
		FileWriter = NULL;

		UE_LOGF(LogCollisionAnalyzer, Log, "Saved collision analyzer data to file '%ls'.", *ProfileFileName);
	}
	else
	{
		UE_LOGF(LogCollisionAnalyzer, Warning, "Unable to save collision analyzer data to file '%ls'.", *ProfileFileName);
	}
}

void FCollisionAnalyzer::LoadCollisionProfileData(FString ProfileFileName)
{
	FArchive* FileReader = IFileManager::Get().CreateFileReader(*ProfileFileName);

	bool bSuccess = false;

	if (FileReader != nullptr)
	{
		FCollisionAnalyzerProxyArchive Ar(*FileReader);

		int32 Magic;
		Ar << Magic;
		if(Magic == COLLISION_ANALYZER_MAGIC)
		{
			int32 Version;
			Ar << Version;
			if(Version == COLLISION_ANALYZER_VERSION)
			{
				// First clear old data
				Queries.Reset();
				DrawQueryIndices.Empty();
				CurrentFrameNum = 0;

				// Load data from file
				Ar << Queries;

				// Invoke event that data has changed
				QueriesChangedEvent.Broadcast();

				UE_LOGF(LogCollisionAnalyzer, Log, "Loaded collision analyzer data from file '%ls'.", *ProfileFileName);

				bSuccess = true;
			}
		}

		// Close file
		FileReader->Close();
		delete FileReader;
		FileReader = NULL;
	}
	
	if(!bSuccess)
	{
		UE_LOGF(LogCollisionAnalyzer, Warning, "Unable to load collision analyzer data from file '%ls'.", *ProfileFileName);
	}
}
