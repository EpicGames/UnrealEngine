// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollisionDebugDrawingPublic.h"

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "CollisionQueryParams.h"
#include "ICollisionAnalyzer.h"
#include "Serialization/NameAsStringProxyArchive.h"

class SWidget;

/** Stores information about one collision query */
struct FCAQuery
{
	FCAQuery();
	~FCAQuery();

	FVector						Start;
	FVector						End;
	FQuat						Rot;
	ECAQueryType::Type			Type;
	ECAQueryShape::Type			Shape;
	ECAQueryMode::Type			Mode;
	FVector						Dims;
	ECollisionChannel			Channel;
	FCollisionQueryParams		Params;
	FCollisionResponseParams	ResponseParams;
	FCollisionObjectQueryParams	ObjectParams;
	TArray<FHitResult>			Results;
	TArray<FHitResult>			TouchAllResults;
	int32						FrameNum;
	float						CPUTime; /** In ms */
	int32						ID;

	friend FArchive& operator << ( FArchive& Ar, FCAQuery& Query );
};

/** Actual implementation of CollisionAnalyzer, private inside module */
class FCollisionAnalyzer : public ICollisionAnalyzer
{
public:
	//~ Begin ICollisionAnalyzer Interface
	virtual void CaptureQuery(
		const FVector& Start, 
		const FVector& End, 
		const FQuat& Rot, 
		ECAQueryType::Type QueryType, 
		ECAQueryShape::Type QueryShape,
		ECAQueryMode::Type QueryMode,
		const FVector& Dims, 
		ECollisionChannel TraceChannel, 
		const struct FCollisionQueryParams& Params, 
		const FCollisionResponseParams&	ResponseParams,
		const FCollisionObjectQueryParams&	ObjectParams,
		const TArray<FHitResult>& Results, 
		const TArray<FHitResult>& TouchAllResults,
		double CPUTime) override;

	/** Returns a new Collision Analyzer widget. */
	virtual TSharedPtr<SWidget> SummonUI() override;

	virtual void TickAnalyzer(UWorld* InWorld) override;
	virtual bool IsRecording() override;
	//~ End ICollisionAnalyzer Interface

	/** Change the current recording state */
	void SetIsRecording(bool bNewRecording);
	/** Get the current number of frames we have recorded */
	int32 GetNumFramesOfRecording();

	FCollisionAnalyzer()
		: DrawBox(ForceInit)		
		, CurrentFrameNum(0)
		, bIsRecording(false)
	{
	}

	virtual ~FCollisionAnalyzer() = default; 

	/** All collected query data */
	TArray<FCAQuery>	Queries;
	/** Indices of queries in Queries array that we want to draw in 3D */
	TArray<int32>		DrawQueryIndices;
	/** AABB to draw in the world */
	FBox				DrawBox;

	DECLARE_EVENT( FCollisionAnalyzer, FQueriesChangedEvent );
	FQueriesChangedEvent& OnQueriesChanged() { return QueriesChangedEvent; }
	FQueriesChangedEvent& OnQueryAdded() { return QueryAddedEvent; }

	/** Save current data to a file */
	void SaveCollisionProfileData(FString ProfileFileName);
	/** Load data from file */
	void LoadCollisionProfileData(FString ProfileFileName);

private:
	/** The current frame number we are on while recording */
	int32				CurrentFrameNum;
	/** Whether we are currently recording */
	bool				bIsRecording;
	/** Event called when Queries array changes */
	FQueriesChangedEvent QueriesChangedEvent;
	/** Event called when a single query is added to array */
	FQueriesChangedEvent QueryAddedEvent;

};

/**
 * Local archive that implements FWeakObjectPtr serialisation.
 * In an ideal world we'd combine FNameAsStringProxyArchive and
 * FArchiveUObject in a generic way, but the duplicate
 * inheritance issues aren't worth it here.
 */
struct FCollisionAnalyzerProxyArchive : public FNameAsStringProxyArchive
{
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInnerArchive The inner archive to proxy.
	 */
	 FCollisionAnalyzerProxyArchive(FArchive& InInnerArchive)
		 :	FNameAsStringProxyArchive(InInnerArchive)
	 { }

	 using FNameAsStringProxyArchive::operator<<;

	 virtual FArchive& operator<< (struct FWeakObjectPtr& Value) override
	 {
		 return FArchiveUObject::SerializeWeakObjectPtr(*this, Value);
	 }
};
