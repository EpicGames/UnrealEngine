// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollisionAnalyzerModule.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "CollisionAnalyzer.h"
#include "CollisionAnalyzerStyle.h"
#include "CollisionAnalyzerLog.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"

namespace CollisionAnalyzerModule
{
	static const FName CollisionAnalyzerApp = FName(TEXT("CollisionAnalyzerApp"));
}

IMPLEMENT_MODULE(FCollisionAnalyzerModule, CollisionAnalyzer);
DEFINE_LOG_CATEGORY(LogCollisionAnalyzer);

void FCollisionAnalyzerModule::StartupModule() 
{
	FCollisionAnalyzerStyle::Initialize();

	CollisionAnalyzer = new FCollisionAnalyzer();

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(CollisionAnalyzerModule::CollisionAnalyzerApp, FOnSpawnTab::CreateRaw(this, &FCollisionAnalyzerModule::SpawnCollisionAnalyzerTab))
		.SetDisplayName(NSLOCTEXT("CollisionAnalyzerModule", "TabTitle", "Collision Analyzer"))
		.SetTooltipText(NSLOCTEXT("CollisionAnalyzerModule", "TooltipText", "Open the Collision Analyzer tab."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "CollisionAnalyzer.TabIcon"));
}

void FCollisionAnalyzerModule::ShutdownModule() 
{
	if (CollisionAnalyzer != NULL)
	{
		delete CollisionAnalyzer;
		CollisionAnalyzer = NULL;
	}

	FCollisionAnalyzerStyle::Shutdown();
}

TSharedRef<SDockTab> FCollisionAnalyzerModule::SpawnCollisionAnalyzerTab(const FSpawnTabArgs& Args)
{
	check(CollisionAnalyzer);

	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CollisionAnalyzer->SummonUI().ToSharedRef()
		];
}
