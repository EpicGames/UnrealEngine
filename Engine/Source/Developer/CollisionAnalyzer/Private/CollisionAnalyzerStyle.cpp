// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollisionAnalyzerStyle.h"
#include "Styling/SlateTypes.h"
#include "Engine/GameEngine.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"

TSharedPtr< FCollisionAnalyzerStyle::FStyle > FCollisionAnalyzerStyle::StyleInstance;

FCollisionAnalyzerStyle::FStyle::FStyle()
	: FSlateStyleSet("CollisionAnalyzerStyle")
{

}

void FCollisionAnalyzerStyle::FStyle::Initialize()
{
	StyleInstance->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleInstance->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FLinearColor SelectionColor(0.728f, 0.364f, 0.003f);
	const FLinearColor SelectionColor_Pressed(0.701f, 0.225f, 0.003f);

	const FCheckBoxStyle ToggleButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(1, 1, 1, 0.1f)))
		.SetUncheckedHoveredImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
		.SetUncheckedPressedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
		.SetCheckedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
		.SetCheckedHoveredImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
		.SetCheckedPressedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed));
	StyleInstance->Set("ToggleButtonCheckbox", ToggleButtonStyle);

	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);

	const FButtonStyle CommonButtonStyle = FButtonStyle()
		.SetNormal(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(1, 1, 1, 0.1f)))
		.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
		.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed));
	StyleInstance->Set("CommonButton", CommonButtonStyle);

	StyleInstance->Set("CollisionAnalyzer.TabIcon", new IMAGE_BRUSH("Icons/icon_tab_CollisionAnalyser_16x", Icon16x16));
	StyleInstance->Set("CollisionAnalyzer.Record", new IMAGE_BRUSH("Icons/CA_Record", Icon24x24));
	StyleInstance->Set("CollisionAnalyzer.Stop", new IMAGE_BRUSH("Icons/CA_Stop", Icon24x24));
	StyleInstance->Set("CollisionAnalyzer.ShowRecent", new IMAGE_BRUSH("Icons/CA_ShowRecent", Icon24x24));
	StyleInstance->Set("CollisionAnalyzer.Group", new IMAGE_BRUSH("Icons/CA_Group", FVector2D(10, 18)));
	StyleInstance->Set("CollisionAnalyzer.GroupBackground", new BOX_BRUSH("Icons/CA_GroupBackground", FMargin(4.f / 16.f)));
	StyleInstance->Set("CollisionAnalyzer.Save", new IMAGE_BRUSH("Icons/icon_file_save_40x", Icon24x24));
	StyleInstance->Set("CollisionAnalyzer.Load", new IMAGE_BRUSH("Icons/icon_file_open_40x", Icon24x24));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance.Get());

	SetParentStyleName(FAppStyle::GetAppStyleSetName());
}


void FCollisionAnalyzerStyle::Initialize()
{
	StyleInstance = MakeShareable(new FCollisionAnalyzerStyle::FStyle);
	StyleInstance->Initialize();
}

void FCollisionAnalyzerStyle::Shutdown()
{
	if (StyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance.Get());
		ensure(StyleInstance.IsUnique());
		StyleInstance.Reset();
	}
}

TSharedPtr< ISlateStyle > FCollisionAnalyzerStyle::Get()
{
	return StyleInstance;
}
