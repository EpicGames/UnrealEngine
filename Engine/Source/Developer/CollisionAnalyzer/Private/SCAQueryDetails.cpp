// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCAQueryDetails.h"
#include "Components/PrimitiveComponent.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "CollisionAnalyzerStyle.h"
#include "SCollisionAnalyzer.h"

#define LOCTEXT_NAMESPACE "SCAQueryDetails"

/** Util to give written explanation for why we missed something */
FText GetReasonForMiss(const UPrimitiveComponent* MissedComp, const FCAQuery* Query)
{
	if(MissedComp != NULL && Query != NULL)
	{
		if(MissedComp->GetOwner() && !MissedComp->GetOwner()->GetActorEnableCollision())
		{
			return FText::Format(LOCTEXT("MissReasonActorCollisionDisabledFmt", "Owning Actor '{0}' has all collision disabled (SetActorEnableCollision)"), FText::FromString(MissedComp->GetOwner()->GetName()));
		}

		if(!MissedComp->IsCollisionEnabled())
		{
			return FText::Format(LOCTEXT("MissReasonComponentCollisionDisabledFmt", "Component '{0}' has CollisionEnabled == NoCollision"), FText::FromString(MissedComp->GetName()));
		}

		if(MissedComp->GetCollisionResponseToChannel(Query->Channel) == ECR_Ignore)
		{
			return FText::Format(LOCTEXT("MissReasonComponentIgnoresChannelFmt", "Component '{0}' ignores this channel."), FText::FromString(MissedComp->GetName()));
		}

		if(Query->ResponseParams.CollisionResponse.GetResponse(MissedComp->GetCollisionObjectType()) == ECR_Ignore)
		{
			return FText::Format(LOCTEXT("MissReasonQueryIgnoresComponentFmt", "Query ignores Component '{0}' movement channel."), FText::FromString(MissedComp->GetName()));
		}
	}

	return LOCTEXT("MissReasonUnknown", "Unknown");
}

/** Implements a row widget for result list. */
class SHitResultRow : public SMultiColumnTableRow< TSharedPtr<FCAHitInfo> >
{
public:

	SLATE_BEGIN_ARGS(SHitResultRow) {}
		SLATE_ARGUMENT(TSharedPtr<FCAHitInfo>, Info)
		SLATE_ARGUMENT(TSharedPtr<SCAQueryDetails>, OwnerDetailsPtr)
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Info = InArgs._Info;
		OwnerDetailsPtr = InArgs._OwnerDetailsPtr;
		SMultiColumnTableRow< TSharedPtr<FCAHitInfo> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		// Get info to apply to all columns (color and tooltip)
		FSlateColor ResultColor = FSlateColor::UseForeground();
		FText TooltipText = FText::GetEmpty();
		if(Info->bMiss)
		{
			ResultColor = FLinearColor(0.4f,0.4f,0.65f);
			TooltipText = FText::Format(LOCTEXT("MissToolTipFmt", "Miss: {0}"), GetReasonForMiss(Info->Result.Component.Get(), OwnerDetailsPtr.Pin()->GetCurrentQuery()));
		}
		else if(Info->Result.bBlockingHit && Info->Result.bStartPenetrating)
		{
			ResultColor = FLinearColor(1.f,0.25f,0.25f);
		}


		// Generate widget for column
		if (ColumnName == TEXT("Time"))
		{
			static const FNumberFormattingOptions TimeNumberFormat = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(3)
				.SetMaximumFractionalDigits(3);

			return	SNew(STextBlock)
					.ColorAndOpacity( ResultColor )
					.ToolTipText( TooltipText )
					.Text( FText::AsNumber(Info->Result.Time, &TimeNumberFormat) );
		}
		else if (ColumnName == TEXT("Type"))
		{
			FText TypeText = FText::GetEmpty();
			if(Info->bMiss)
			{
				TypeText = LOCTEXT("MissLabel", "Miss");
			}
			else if(Info->Result.bBlockingHit)
			{
				TypeText = LOCTEXT("BlockLabel", "Block");
			}
			else
			{
				TypeText = LOCTEXT("TouchLabel", "Touch");
			}

			return	SNew(STextBlock)
					.ColorAndOpacity( ResultColor )
					.ToolTipText( TooltipText )
					.Text( TypeText );
		}
		else if (ColumnName == TEXT("Component"))
		{
			FText LongName = LOCTEXT("InvalidLabel", "Invalid");
			if(Info->Result.Component.IsValid())
			{
				LongName = FText::FromString(Info->Result.Component.Get()->GetReadableName());
			}

			return	SNew(STextBlock)
					.ColorAndOpacity( ResultColor )
					.ToolTipText( TooltipText )
					.Text( LongName );
		}
		else if (ColumnName == TEXT("Normal"))
		{
			return	SNew(STextBlock)
					.ColorAndOpacity( ResultColor )
					.ToolTipText( TooltipText )
					.Text( FText::FromString(Info->Result.Normal.ToString()) );
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** Result to display */
	TSharedPtr<FCAHitInfo> Info;
	/** Show details of  */
	TWeakPtr<SCAQueryDetails> OwnerDetailsPtr;
};


//////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCAQueryDetails::Construct(const FArguments& InArgs, TSharedPtr<SCollisionAnalyzer> OwningAnalyzerWidget)
{
	bDisplayQuery = false;
	bShowMisses = false;
	OwningAnalyzerWidgetPtr = OwningAnalyzerWidget;

	ChildSlot
	[
		SNew(SVerticalBox)
		// Top area is info on the trace
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SBorder )
			.BorderImage(FCollisionAnalyzerStyle::Get()->GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				// Left is start/end locations
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SGridPanel)
					+SGridPanel::Slot(0,0)
					.Padding(2)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("QueryStart", "Start:"))
					]
					+SGridPanel::Slot(1,0)
					.Padding(2)
					[
						SNew(STextBlock)
						.Text(this, &SCAQueryDetails::GetStartText)
					]
					+SGridPanel::Slot(0,1)
					.Padding(2)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("QueryEnd", "End:"))
					]
					+SGridPanel::Slot(1,1)
					.Padding(2)
					[
						SNew(STextBlock)
						.Text(this, &SCAQueryDetails::GetEndText)
					]
				]
				// Right has controls
				+SHorizontalBox::Slot()
				.FillWidth(1)
				.VAlign(VAlign_Top)
				.Padding(4,0)
				[
					SNew(SCheckBox)
					.OnCheckStateChanged(this, &SCAQueryDetails::OnToggleShowMisses)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ShowMisses", "Show Misses"))
					]
				]
			]
		]
		// Bottom area is list of hits
		+SVerticalBox::Slot()
		.FillHeight(1) 
		[
			SNew(SBorder)
			.BorderImage(FCollisionAnalyzerStyle::Get()->GetBrush("Menu.Background"))
			.Padding(1.0)
			[
				SAssignNew(ResultListWidget, SListView< TSharedPtr<FCAHitInfo> >)
				.ListItemsSource(&ResultList)
				.SelectionMode(ESelectionMode::Single)
				.OnSelectionChanged(this, &SCAQueryDetails::ResultListSelectionChanged)
				.OnGenerateRow(this, &SCAQueryDetails::ResultListGenerateRow)
				.HeaderRow(
					SNew(SHeaderRow)
					+SHeaderRow::Column("Time").DefaultLabel(LOCTEXT("ResultListTimeHeader", "Time")).FillWidth(0.7)
					+SHeaderRow::Column("Type").DefaultLabel(LOCTEXT("ResultListTypeHeader", "Type")).FillWidth(0.7)
					+SHeaderRow::Column("Component").DefaultLabel(LOCTEXT("ResultListComponentHeader", "Component")).FillWidth(3)
					+SHeaderRow::Column("Normal").DefaultLabel(LOCTEXT("ResultListNormalHeader", "Normal")).FillWidth(1.8)
				)
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SCAQueryDetails::GetStartText() const
{
	return bDisplayQuery ? CurrentQuery.Start.ToText() : FText::GetEmpty();
}

FText SCAQueryDetails::GetEndText() const
{
	return bDisplayQuery ? CurrentQuery.End.ToText() : FText::GetEmpty();
}

TSharedRef<ITableRow> SCAQueryDetails::ResultListGenerateRow(TSharedPtr<FCAHitInfo> Info, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SHitResultRow, OwnerTable)
		.Info(Info)
		.OwnerDetailsPtr(SharedThis(this));
}

void SCAQueryDetails::UpdateDisplayedBox()
{
	FCollisionAnalyzer* Analyzer = OwningAnalyzerWidgetPtr.Pin()->Analyzer;
	Analyzer->DrawBox = FBox(ForceInit);

	if(bDisplayQuery)
	{
		TArray< TSharedPtr<FCAHitInfo> > SelectedInfos = ResultListWidget->GetSelectedItems();
		if(SelectedInfos.Num() > 0)
		{
			UPrimitiveComponent* HitComp = SelectedInfos[0]->Result.Component.Get();
			if(HitComp != NULL)
			{
				Analyzer->DrawBox = HitComp->Bounds.GetBox();
			}
		}
	}
}


void SCAQueryDetails::ResultListSelectionChanged(TSharedPtr<FCAHitInfo> SelectedInfos, ESelectInfo::Type SelectInfo)
{
	UpdateDisplayedBox();
}


void SCAQueryDetails::OnToggleShowMisses(ECheckBoxState InCheckboxState)
{
	bShowMisses = (InCheckboxState == ECheckBoxState::Checked);
	UpdateResultList();
}

ECheckBoxState SCAQueryDetails::GetShowMissesState() const
{
	return bShowMisses ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


/** See if an array of results contains a particular component */
static bool ResultsContainComponent(const TArray<FHitResult>& Results, UPrimitiveComponent* Component)
{
	for(int32 i=0; i<Results.Num(); i++)
	{
		if(Results[i].Component.Get() == Component)
		{
			return true;
		}
	}
	return false;
}

void SCAQueryDetails::UpdateResultList()
{
	ResultList.Empty();
	UpdateDisplayedBox();

	if(bDisplayQuery)
	{
		// First add actual results
		for(int32 i=0; i<CurrentQuery.Results.Num(); i++)
		{
			ResultList.Add( FCAHitInfo::Make(CurrentQuery.Results[i], false) );
		}

		// If desired, look for results from our touching query that were not in the real results, and add them
		if(bShowMisses)
		{
			for(int32 i=0; i<CurrentQuery.TouchAllResults.Num(); i++)
			{
				FHitResult& MissResult = CurrentQuery.TouchAllResults[i];
				if(MissResult.Component.IsValid() && !ResultsContainComponent(CurrentQuery.Results, MissResult.Component.Get()))
				{
					ResultList.Add( FCAHitInfo::Make(MissResult, true) );
				}
			}
		}

		// Then sort
		struct FCompareFCAHitInfo
		{
			FORCEINLINE bool operator()( const TSharedPtr<FCAHitInfo> A, const TSharedPtr<FCAHitInfo> B ) const
			{
				check(A.IsValid());
				check(B.IsValid());
				return A->Result.Time < B->Result.Time;
			}
		};

		ResultList.Sort( FCompareFCAHitInfo() );
	}

	// Finally refresh display widget
	ResultListWidget->RequestListRefresh();
}

void SCAQueryDetails::SetCurrentQuery(const FCAQuery& NewQuery)
{
	bDisplayQuery = true;
	CurrentQuery = NewQuery;
	UpdateResultList();	
}

void SCAQueryDetails::ClearCurrentQuery()
{
	bDisplayQuery = false;
	ResultList.Empty();
	UpdateDisplayedBox();
}

FCAQuery* SCAQueryDetails::GetCurrentQuery()
{
	return bDisplayQuery ? &CurrentQuery : NULL;
}

#undef LOCTEXT_NAMESPACE
