// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/HitResult.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "CollisionAnalyzer.h"

class SCollisionAnalyzer;

/** Info about a hit */
class FCAHitInfo
{
public:
	FHitResult	Result;
	bool		bMiss;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FCAHitInfo> Make(FHitResult& Result, bool bMiss)
	{
		TSharedRef<FCAHitInfo> NewItem = MakeShareable(new FCAHitInfo(Result, bMiss));
		return NewItem;
	}

private:
	FCAHitInfo(FHitResult& InResult, bool bInMiss)
		: Result(InResult)
		, bMiss(bInMiss)
	{}
};

/** Widget to display details about a single query */
class SCAQueryDetails : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCAQueryDetails) {}
	SLATE_END_ARGS()


public:

	void Construct(const FArguments& InArgs, TSharedPtr<SCollisionAnalyzer> OwningAnalyzerWidget);

	/** Set the current query to display */
	void SetCurrentQuery(const FCAQuery& NewQuery);
	/** Show no query */
	void ClearCurrentQuery();
	/** Get current query */
	FCAQuery* GetCurrentQuery();

	// Get delegates
	FText GetStartText() const;
	FText GetEndText() const;
	ECheckBoxState GetShowMissesState() const;
	// List delegates
	TSharedRef<ITableRow> ResultListGenerateRow(TSharedPtr<FCAHitInfo> Info, const TSharedRef<STableViewBase>& OwnerTable);
	void ResultListSelectionChanged(TSharedPtr<FCAHitInfo> SelectedInfos, ESelectInfo::Type SelectInfo);
	// Handler delegates
	void OnToggleShowMisses(ECheckBoxState InCheckboxState);

private:

	/** Update ResultList from CurrentQuery */
	void UpdateResultList();
	/** Update the box we are drawing */
	void UpdateDisplayedBox();

	// MEMBERS
	/** Owning SCollisionAnalyzer */
	TWeakPtr<SCollisionAnalyzer>	OwningAnalyzerWidgetPtr;

	/** Are we currently displaying a query */
	bool		bDisplayQuery;
	/** Current query we are displaying */
	FCAQuery	CurrentQuery;
	/** Array used by list widget, just a copy of that in CurrentQuery */
	TArray< TSharedPtr<FCAHitInfo> >	ResultList;
	/** */
	bool		bShowMisses;

	// WIDGETS

	TSharedPtr< SListView< TSharedPtr<FCAHitInfo> > > ResultListWidget;
};
