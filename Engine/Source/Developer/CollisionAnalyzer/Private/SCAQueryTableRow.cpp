// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCAQueryTableRow.h"
#include "CollisionAnalyzerStyle.h"
#include "Engine/HitResult.h"

#define LOCTEXT_NAMESPACE "CollisionAnalyzer"

void SCAQueryTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	OwnerAnalyzerWidgetPtr = InArgs._OwnerAnalyzerWidget;
	SMultiColumnTableRow< TSharedPtr<FQueryTreeItem> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);

	if(Item->bIsGroup)
	{
		SetBorderImage(FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.GroupBackground"));
	}
}

TSharedRef<SWidget> SCAQueryTableRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<SCollisionAnalyzer> OwnerAnalyzerWidget = OwnerAnalyzerWidgetPtr.Pin();

	// GROUP
	if(Item->bIsGroup)
	{
		if (ColumnName == SCollisionAnalyzer::IDColumnName)
		{
			return	SNew(SExpanderArrow, SharedThis(this));
		}
		else if (ColumnName == SCollisionAnalyzer::FrameColumnName && OwnerAnalyzerWidget->GroupBy == EQueryGroupMode::ByFrameNum)
		{
			return	SNew(STextBlock)
					.Font(FCollisionAnalyzerStyle::Get()->GetFontStyle("BoldFont"))
					.Text(FText::AsNumber(Item->FrameNum));
		}
		else if (ColumnName == SCollisionAnalyzer::TagColumnName && OwnerAnalyzerWidget->GroupBy == EQueryGroupMode::ByTag)
		{
			return	SNew(STextBlock)
					.Font(FCollisionAnalyzerStyle::Get()->GetFontStyle("BoldFont"))
					.Text(FText::FromName(Item->GroupName));
		}
		else if (ColumnName == SCollisionAnalyzer::OwnerColumnName && OwnerAnalyzerWidget->GroupBy == EQueryGroupMode::ByOwnerTag)
		{
			return	SNew(STextBlock)
					.Font(FCollisionAnalyzerStyle::Get()->GetFontStyle("BoldFont"))
					.Text(FText::FromName(Item->GroupName));
		}
		else if (ColumnName == SCollisionAnalyzer::TimeColumnName)
		{
			return	SNew(STextBlock)
					.Font(FCollisionAnalyzerStyle::Get()->GetFontStyle("BoldFont"))
					.Text(this, &SCAQueryTableRow::GetTotalTimeText);
		}
	}
	// ITEM
	else
	{
		const int32 QueryId = Item->QueryIndex;
		if(QueryId >= OwnerAnalyzerWidget->Analyzer->Queries.Num())
		{
			return	SNew(STextBlock)
					.Text( LOCTEXT("ErrorMessage", "ERROR") );
		}

		FCAQuery& Query = OwnerAnalyzerWidget->Analyzer->Queries[QueryId];

		if (ColumnName == SCollisionAnalyzer::IDColumnName)
		{	
			return	SNew(STextBlock)
					.Text( FText::AsNumber(QueryId) );
		}
		else if (ColumnName == SCollisionAnalyzer::FrameColumnName)
		{
			return	SNew(STextBlock)
					.Text( FText::AsNumber(Query.FrameNum) );
		}
		else if (ColumnName == SCollisionAnalyzer::TypeColumnName)
		{
			return	SNew(STextBlock)
					.Text(FText::FromString(SCollisionAnalyzer::QueryTypeToString(Query.Type)));
		}
		else if (ColumnName == SCollisionAnalyzer::ShapeColumnName)
		{
			// Leave shape string blank if this is a raycast, it doesn't matter
			FString ShapeString;
			if(Query.Type != ECAQueryType::Raycast)
			{
				ShapeString = SCollisionAnalyzer::QueryShapeToString(Query.Shape);
			}

			return	SNew(STextBlock)
					.Text(FText::FromString(ShapeString));
		}
		else if (ColumnName == SCollisionAnalyzer::ModeColumnName)
		{
			return	SNew(STextBlock)
					.Text(FText::FromString(SCollisionAnalyzer::QueryModeToString(Query.Mode)));
		}
		else if (ColumnName == SCollisionAnalyzer::TagColumnName)
		{
			return	SNew(STextBlock)
					.Text(FText::FromName(Query.Params.TraceTag));
		}
		else if (ColumnName == SCollisionAnalyzer::OwnerColumnName)
		{
			return	SNew(STextBlock)
					.Text(FText::FromName(Query.Params.OwnerTag));
		}
		else if (ColumnName == SCollisionAnalyzer::NumBlockColumnName)
		{
			FHitResult* FirstHit = FHitResult::GetFirstBlockingHit(Query.Results);
			bool bStartPenetrating = (FirstHit != NULL) && FirstHit->bStartPenetrating;
			// Draw number in red if we start penetrating
			return	SNew(STextBlock)
					.Text(FText::AsNumber(FHitResult::GetNumBlockingHits(Query.Results)))
					.ColorAndOpacity(bStartPenetrating ? FLinearColor(1.f,0.25f,0.25f) : FSlateColor::UseForeground() );
		}
		else if (ColumnName == SCollisionAnalyzer::NumTouchColumnName)
		{
			return	SNew(STextBlock)
					.Text(FText::AsNumber(FHitResult::GetNumOverlapHits(Query.Results)));
		}
		else if (ColumnName == SCollisionAnalyzer::TimeColumnName)
		{
			return	SNew(STextBlock)
					.Text(FText::FromString(FString::Printf(TEXT("%.3f"), Query.CPUTime)));
		}
	}

	return SNullWidget::NullWidget;
}

FText SCAQueryTableRow::GetTotalTimeText() const
{
	check(Item->bIsGroup)
	return FText::FromString(FString::Printf(TEXT("%.3f"), Item->TotalCPUTime));
}

#undef LOCTEXT_NAMESPACE
