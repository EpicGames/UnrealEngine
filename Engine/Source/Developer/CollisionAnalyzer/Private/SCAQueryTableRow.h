// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "SCollisionAnalyzer.h"

/** Implements a row widget for query list. */
class SCAQueryTableRow : public SMultiColumnTableRow< TSharedPtr<class FQueryTreeItem> >
{
public:

	SLATE_BEGIN_ARGS(SCAQueryTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<class SCollisionAnalyzer>, OwnerAnalyzerWidget)
		SLATE_ARGUMENT(TSharedPtr<class FQueryTreeItem>, Item)
	SLATE_END_ARGS()


public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	// delegate
	FText GetTotalTimeText() const;

private:

	/** Tree item */
	TSharedPtr<class FQueryTreeItem>	Item;
	/** Analyzer widget that owns us */
	TWeakPtr<SCollisionAnalyzer>		OwnerAnalyzerWidgetPtr;
};
