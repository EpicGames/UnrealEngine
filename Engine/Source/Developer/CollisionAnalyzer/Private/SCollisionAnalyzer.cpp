// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCollisionAnalyzer.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Input/SCheckBox.h"
#include "CollisionAnalyzerStyle.h"
#include "SCAQueryTableRow.h"
#include "SCAQueryDetails.h"
#include "DesktopPlatformModule.h"

#define LOCTEXT_NAMESPACE "SCollisionAnalyzer"

static const int32 NumDrawRecentQueries = 10;

// Column names
const FName SCollisionAnalyzer::IDColumnName(TEXT("ID"));
const FName SCollisionAnalyzer::FrameColumnName(TEXT("Frame"));
const FName SCollisionAnalyzer::TypeColumnName(TEXT("Type"));
const FName SCollisionAnalyzer::ShapeColumnName(TEXT("Shape"));
const FName SCollisionAnalyzer::ModeColumnName(TEXT("Mode"));
const FName SCollisionAnalyzer::TagColumnName(TEXT("Tag"));
const FName SCollisionAnalyzer::OwnerColumnName(TEXT("Owner"));
const FName SCollisionAnalyzer::NumBlockColumnName(TEXT("NumBlock"));
const FName SCollisionAnalyzer::NumTouchColumnName(TEXT("NumTouch"));
const FName SCollisionAnalyzer::TimeColumnName(TEXT("Time"));

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCollisionAnalyzer::Construct(const FArguments& InArgs, FCollisionAnalyzer* InAnalyzer)
{
	Analyzer = InAnalyzer;
	bDrawRecentQueries = false;
	FrameFilterNum = INDEX_NONE;
	MinCPUFilterTime = -1.f;
	GroupBy = EQueryGroupMode::Ungrouped;
	SortBy = EQuerySortMode::ByID;
	SortDirection = EColumnSortMode::Descending;
	TotalNumQueries = 0;

	ChildSlot
	[
		SNew(SVerticalBox)
		// Toolbar
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SBorder )
			.BorderImage( FCollisionAnalyzerStyle::Get()->GetBrush( "ToolPanel.GroupBorder" ) )
			[
				SNew(SHorizontalBox)
				// Record button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1)
				[
					SNew(SButton)
					.ButtonStyle(FCollisionAnalyzerStyle::Get(), "CommonButton")
					.OnClicked(this, &SCollisionAnalyzer::OnRecordButtonClicked)
					.ToolTipText(LOCTEXT("CAnalyzerRecord_Tooltip", "Start/stop recording collision data"))
					.Content()
					[
						SNew(SImage)
						.Image(this, &SCollisionAnalyzer::GetRecordButtonBrush)
					]
				]
				// 'Draw most recent' toggle button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1)
				[
					SNew(SCheckBox)
					.Style(FCollisionAnalyzerStyle::Get(), "ToggleButtonCheckbox")
					.ToolTipText(LOCTEXT("CAnalyzerToggleRecent_Tooltip", "Toggles drawing of recent collision data"))
					.OnCheckStateChanged(this, &SCollisionAnalyzer::OnDrawRecentChanged)
					.IsChecked(this, &SCollisionAnalyzer::GetDrawRecentState)
					.Content()
					[
						SNew(SImage)
						.Image(FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.ShowRecent"))
					]
				]
				// Load profile
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1)
				[
					SNew(SButton)
					.ButtonStyle(FCollisionAnalyzerStyle::Get(), "CommonButton")
					.OnClicked(this, &SCollisionAnalyzer::OnLoadButtonClicked)
					.ToolTipText(LOCTEXT("CAnalyzerLoad_Tooltip", "Load a previously recorded collision data session"))
					.Content()
					[
						SNew(SImage)
						.Image(FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.Load"))
					]
				]
				// Save profile
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1)
				[
					SNew(SButton)
					.ButtonStyle(FCollisionAnalyzerStyle::Get(), "CommonButton")
					.OnClicked(this, &SCollisionAnalyzer::OnSaveButtonClicked)
					.ToolTipText(LOCTEXT("CAnalyzerSave_Tooltip", "Save the current recorded collision data"))
					.Content()
					[
						SNew(SImage)
						.Image(FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.Save"))
					]
				]
			]
		]
		// List area
		+SVerticalBox::Slot()
		.FillHeight(1) 
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Vertical)
			+SSplitter::Slot()
			.Value(2)
			[
				SNew(SBorder)
				.BorderImage(FCollisionAnalyzerStyle::Get()->GetBrush("Menu.Background"))
				.Padding(1.0)
				[
					SAssignNew(QueryTreeWidget, STreeView< TSharedPtr<FQueryTreeItem> >)
					.TreeItemsSource(&GroupedQueries)
					.SelectionMode(ESelectionMode::Multi)
					.OnGenerateRow(this, &SCollisionAnalyzer::QueryTreeGenerateRow)
					.OnSelectionChanged(this, &SCollisionAnalyzer::QueryTreeSelectionChanged)
					.OnGetChildren(this, &SCollisionAnalyzer::OnGetChildrenForQueryGroup)
					.HeaderRow(
						SNew(SHeaderRow)
						// ID
						+SHeaderRow::Column(IDColumnName)
						.SortMode(this, &SCollisionAnalyzer::GetIDSortMode)
						.OnSort(this, &SCollisionAnalyzer::OnSortByChanged)
						.HAlignCell(HAlign_Left)
						.FixedWidth(48)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("QueryListIdHeader", "ID"))
						]
						// Frame number
						+SHeaderRow::Column(FrameColumnName)
						.FixedWidth(48)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("QueryListFrameHeader", "Frame"))
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0,2)
							[
								SNew(SHorizontalBox)
								// Filter entry
								+SHorizontalBox::Slot()
								.FillWidth(1)
								[
									SAssignNew(FrameFilterBox, SEditableTextBox)
									.SelectAllTextWhenFocused(true)
									.OnTextCommitted(this, &SCollisionAnalyzer::FilterTextCommitted)
								]
								// Group toggle
								+SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.Style(FCollisionAnalyzerStyle::Get(), "ToggleButtonCheckbox")
									.OnCheckStateChanged(this, &SCollisionAnalyzer::OnGroupByFrameChanged)
									.IsChecked( this, &SCollisionAnalyzer::GetGroupByFrameState)
									.Content()
									[
										SNew(SImage)
										.Image(FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.Group"))
									]
								]
							]
						]
						// Type
						+SHeaderRow::Column(TypeColumnName)
						.FillWidth(0.5)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("QueryListTypeHeader", "Type"))
						]
						// Shape
						+SHeaderRow::Column(ShapeColumnName)
						.FillWidth(0.5)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("QueryListShapeHeader", "Shape"))
						]
						// Shape
						+SHeaderRow::Column(ModeColumnName)
						.FillWidth(0.5)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("QueryListModeHeader", "Mode"))
						]
						// Tag
						+SHeaderRow::Column(TagColumnName)
						.FillWidth(1.5)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("QueryListTagHeader", "Tag"))
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0,2)
							[
								SNew(SHorizontalBox)
								// Filter entry
								+SHorizontalBox::Slot()
								.FillWidth(1)
								[
									SAssignNew(TagFilterBox, SEditableTextBox)
									.SelectAllTextWhenFocused(true)
									.OnTextCommitted(this, &SCollisionAnalyzer::FilterTextCommitted)
								]
								// Group toggle
								+SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.Style(FCollisionAnalyzerStyle::Get(), "ToggleButtonCheckbox")
									.OnCheckStateChanged(this, &SCollisionAnalyzer::OnGroupByTagChanged)
									.IsChecked( this, &SCollisionAnalyzer::GetGroupByTagState)
									.Content()
									[
										SNew(SImage)
										.Image(FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.Group"))
									]
								]
							]
						]
						// Owner
						+SHeaderRow::Column(OwnerColumnName)
						.FillWidth(1.5)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("QueryListOwnerHeader", "Owner"))
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0,2)
							[
								SNew(SHorizontalBox)
								// Filter entry
								+SHorizontalBox::Slot()
								.FillWidth(1)
								[
									SAssignNew(OwnerFilterBox, SEditableTextBox)
									.SelectAllTextWhenFocused(true)
									.OnTextCommitted(this, &SCollisionAnalyzer::FilterTextCommitted)
								]
								// Group toggle
								+SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SCheckBox)
									.Style(FCollisionAnalyzerStyle::Get(), "ToggleButtonCheckbox")
									.OnCheckStateChanged(this, &SCollisionAnalyzer::OnGroupByOwnerChanged)
									.IsChecked( this, &SCollisionAnalyzer::GetGroupByOwnerState)
									.Content()
									[
										SNew(SImage)
										.Image(FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.Group"))
									]
								]
							]
						]
						// Num blocking hits
						+SHeaderRow::Column(NumBlockColumnName)
						.FixedWidth(24)
						[
							SNew(STextBlock)
							.Text( LOCTEXT("NumberOfBlockColumnHeader", "#B") )
							.ToolTipText( LOCTEXT("NumberBlocksTooltip", "Number of blocking results, red means 'started penetrating'") )
						]
						// Num touching hits
						+SHeaderRow::Column(NumTouchColumnName)
						.FixedWidth(24)
						[
							SNew(STextBlock)
							.Text( LOCTEXT("NumberOfTouchesColumnHeader", "#T") )
							.ToolTipText( LOCTEXT("NumberTouchTooltip", "Number of touching results") )
						]
						// CPU time
						+SHeaderRow::Column(TimeColumnName)
						.SortMode(this, &SCollisionAnalyzer::GetTimeSortMode)
						.OnSort(this, &SCollisionAnalyzer::OnSortByChanged)
						.FixedWidth(48)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text( LOCTEXT("QueryMillisecondsColumnHeader", "ms") )
								.ToolTipText( LOCTEXT("TimeTooltip", "How long this query took, in ms") )
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0,2)
							[
								SAssignNew(TimeFilterBox, SEditableTextBox)
								.SelectAllTextWhenFocused(true)
								.OnTextCommitted(this, &SCollisionAnalyzer::FilterTextCommitted)
							]
						]
					)
				]
			]
			+SSplitter::Slot()
			.Value(1)
			[
				SAssignNew(QueryDetailsWidget, SCAQueryDetails, SharedThis(this))
			]
		]
		// Status area
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SBorder )
			.BorderImage(FCollisionAnalyzerStyle::Get()->GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(STextBlock)
				.Text(this, &SCollisionAnalyzer::GetStatusText)
			]
		]
	];

	Analyzer->OnQueriesChanged().AddSP(this, &SCollisionAnalyzer::OnQueriesChanged);
	Analyzer->OnQueryAdded().AddSP(this, &SCollisionAnalyzer::OnQueryAdded);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

SCollisionAnalyzer::~SCollisionAnalyzer()
{
	Analyzer->OnQueriesChanged().RemoveAll(this);
	Analyzer->OnQueryAdded().RemoveAll(this);
}



void SCollisionAnalyzer::OnQueriesChanged()
{
	RebuildFilteredList();
	UpdateDrawnQueries();
}

void SCollisionAnalyzer::OnQueryAdded()
{
	const int32 NewQueryIndex = Analyzer->Queries.Num()-1;
	const FCAQuery& NewQuery = Analyzer->Queries[NewQueryIndex];
	if(ShouldDisplayQuery(NewQuery))
	{
		// Passed filter so add to filtered results
		AddQueryToGroupedQueries(NewQueryIndex, true);
	}

	QueryTreeWidget->RequestTreeRefresh();
	UpdateDrawnQueries();
}

void SCollisionAnalyzer::UpdateDrawnQueries()
{
	// First empty 'draw set'
	Analyzer->DrawQueryIndices.Empty();
	// Add those that are selected
	TArray< TSharedPtr<FQueryTreeItem> > SelectedItems = QueryTreeWidget->GetSelectedItems();
	for(int32 i=0; i<SelectedItems.Num(); i++)
	{
		TSharedPtr<FQueryTreeItem> Item = SelectedItems[i];
		if(!Item->bIsGroup)
		{
			Analyzer->DrawQueryIndices.Add(Item->QueryIndex);
		}
	}

	// If selected, draw the most recent NumDrawRecentQueries filtered queries
	if(bDrawRecentQueries)
	{
		for(int32 i=0; i<RecentQueries.Num(); i++)
		{
			Analyzer->DrawQueryIndices.Add(RecentQueries[i]);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// Query Tree



TSharedRef<ITableRow> SCollisionAnalyzer::QueryTreeGenerateRow(TSharedPtr<FQueryTreeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SCAQueryTableRow, OwnerTable)
		.Item(InItem)
		.OwnerAnalyzerWidget(SharedThis(this));
}

void SCollisionAnalyzer::QueryTreeSelectionChanged(TSharedPtr<FQueryTreeItem> SelectedItem, ESelectInfo::Type SelectInfo)
{
	UpdateDrawnQueries();

	// If selecting a single non-group item
	TArray< TSharedPtr<FQueryTreeItem> > SelectedItems = QueryTreeWidget->GetSelectedItems();
	if(SelectedItems.Num() == 1)
	{
		TSharedPtr<FQueryTreeItem> Item = SelectedItems[0];
		if(!Item->bIsGroup)
		{
			FCAQuery& ShowQuery = Analyzer->Queries[Item->QueryIndex];
			QueryDetailsWidget->SetCurrentQuery(ShowQuery);
		}
		else
		{
			QueryDetailsWidget->ClearCurrentQuery();
		}
	}
	else
	{
		QueryDetailsWidget->ClearCurrentQuery();
	}
}

void SCollisionAnalyzer::OnGetChildrenForQueryGroup( TSharedPtr<FQueryTreeItem> InItem, TArray<TSharedPtr<FQueryTreeItem> >& OutChildren )
{
	if(InItem->bIsGroup)
	{
		OutChildren = InItem->QueriesInGroup;
	}
}


bool SCollisionAnalyzer::ShouldDisplayQuery(const FCAQuery& Query)
{
	// Check frame number filter
	if(	FrameFilterNum != INDEX_NONE && 
		Query.FrameNum != FrameFilterNum )
	{
		return false;
	}
	
	// Check tag filter
	if(	TagFilterString.Len() > 0 && 
		!Query.Params.TraceTag.ToString().Contains(TagFilterString))
	{
		return false;
	}

	// Check owner filter
	if(	OwnerFilterString.Len() > 0 && 
		!Query.Params.OwnerTag.ToString().Contains(OwnerFilterString))
	{
		return false;
	}

	// Check query time
	if(	MinCPUFilterTime > 0.f && Query.CPUTime < MinCPUFilterTime )
	{
		return false;
	}

	return true;
}

void SCollisionAnalyzer::UpdateFilterInfo()
{
	// Get frame filter
	FrameFilterNum = INDEX_NONE;

	if(FrameFilterBox->GetText().ToString().Len() > 0)
	{
		FrameFilterNum = FCString::Atoi( *FrameFilterBox->GetText().ToString() );
	}

	// Get tag filter
	TagFilterString = TagFilterBox->GetText().ToString();
	// Get owner filter
	OwnerFilterString = OwnerFilterBox->GetText().ToString();

	MinCPUFilterTime = -1.f;
	FString TimeFilterString = TimeFilterBox->GetText().ToString();
	if(TimeFilterString.Len() > 0)
	{
		MinCPUFilterTime = FCString::Atof(*TimeFilterString);
	}
}

TSharedPtr<FQueryTreeItem> SCollisionAnalyzer::FindQueryGroup(FName InGroupName, int32 InFrameNum)
{
	for(int i=0; i<GroupedQueries.Num(); i++)
	{
		TSharedPtr<FQueryTreeItem> Item = GroupedQueries[i];
		if(Item->bIsGroup && ((InGroupName != NAME_None && InGroupName == Item->GroupName) || (InFrameNum != INDEX_NONE && InFrameNum == Item->FrameNum)))
		{
			return Item;
		}
	}

	return NULL;
}

/** Functor for comparing query by CPU time */
struct FCompareQueryByCPUTime
{
	FCollisionAnalyzer* Analyzer;

	FCompareQueryByCPUTime(FCollisionAnalyzer* InAnalyzer)
		: Analyzer(InAnalyzer)
	{}

	FORCEINLINE bool operator()( const TSharedPtr<FQueryTreeItem> A, const TSharedPtr<FQueryTreeItem> B ) const
	{
		check(A.IsValid());
		check(!A->bIsGroup);
		check(B.IsValid());
		check(!A->bIsGroup);
		const FCAQuery& QueryA = Analyzer->Queries[A->QueryIndex];
		const FCAQuery& QueryB = Analyzer->Queries[B->QueryIndex];

		return ( QueryA.CPUTime > QueryB.CPUTime );
	}
};

/** Functor for comparing query by ID */
struct FCompareQueryByID
{
	FCollisionAnalyzer* Analyzer;
	EColumnSortMode::Type SortMode;
	
	FCompareQueryByID(FCollisionAnalyzer* InAnalyzer, EColumnSortMode::Type InSortMode)
		 : Analyzer(InAnalyzer), SortMode(InSortMode)
	 {}
	
	FORCEINLINE bool operator()(const TSharedPtr<FQueryTreeItem> A, const TSharedPtr<FQueryTreeItem> B) const
	{
		check(A.IsValid());
		check(!A->bIsGroup);
		check(B.IsValid());
		check(!A->bIsGroup);
		const FCAQuery& QueryA = Analyzer->Queries[A->QueryIndex];
		const FCAQuery& QueryB = Analyzer->Queries[B->QueryIndex];
		
		if (SortMode == EColumnSortMode::Descending)
		{
			return (QueryA.ID < QueryB.ID);
		}
		else
		{
			return (QueryA.ID > QueryB.ID);
		}
	}
};


/** Functor for comparing group by CPU time */
struct FCompareGroupByCPUTime
{
	FORCEINLINE bool operator()( const TSharedPtr<FQueryTreeItem> A, const TSharedPtr<FQueryTreeItem> B ) const
	{
		check(A.IsValid());
		check(A->bIsGroup);
		check(B.IsValid());
		check(B->bIsGroup);
		return ( A->TotalCPUTime > B->TotalCPUTime );
	}
};

void SCollisionAnalyzer::AddQueryToGroupedQueries(int32 NewQueryIndex, bool bPerformSort)
{
	TSharedPtr<FQueryTreeItem> NewItem = FQueryTreeItem::MakeQuery(NewQueryIndex);
	const FCAQuery& Query = Analyzer->Queries[NewQueryIndex];

	// If not grouping, easy, just add to root list
	if(GroupBy == EQueryGroupMode::Ungrouped)
	{
		GroupedQueries.Add(NewItem);

		if(bPerformSort)
		{
			if(SortBy == EQuerySortMode::ByTime)
			{
				GroupedQueries.Sort( FCompareQueryByCPUTime(Analyzer) );
			}
			else if (SortBy == EQuerySortMode::ByID)
			{
				GroupedQueries.Sort( FCompareQueryByID(Analyzer, SortDirection) );
			}
		}
	}
	// If we are grouping..
	else 
	{
		// .. first find the existing group that we belong to
		FName GroupName = NAME_None;
		int32 FrameNum = INDEX_NONE;
		if(GroupBy == EQueryGroupMode::ByTag || GroupBy == EQueryGroupMode::ByOwnerTag)
		{
			GroupName = (GroupBy == EQueryGroupMode::ByTag) ? Query.Params.TraceTag : Query.Params.OwnerTag;
		}
		else if(GroupBy == EQueryGroupMode::ByFrameNum)
		{
			FrameNum = Query.FrameNum;
		}

		TSharedPtr<FQueryTreeItem> AddToGroup = FindQueryGroup(GroupName, FrameNum);
		// If we failed to find it - make group now
		if(!AddToGroup.IsValid())
		{
			AddToGroup = FQueryTreeItem::MakeGroup(GroupName, FrameNum);
			GroupedQueries.Add(AddToGroup);
		}

		// Finally, add item to that group
		AddToGroup->QueriesInGroup.Add(NewItem);
		AddToGroup->UpdateTotalCPUTime(Analyzer); // update total CPU time

		if(bPerformSort)
		{
			if(SortBy == EQuerySortMode::ByTime)
			{
				GroupedQueries.Sort( FCompareGroupByCPUTime() );
				AddToGroup->QueriesInGroup.Sort( FCompareQueryByCPUTime(Analyzer) );
			}
			else if (SortBy == EQuerySortMode::ByID)
			{
				GroupedQueries.Sort( FCompareGroupByCPUTime() );
				AddToGroup->QueriesInGroup.Sort( FCompareQueryByID(Analyzer, SortDirection) );
			}
		}
	}

	// Update list of recent queries
	RecentQueries.Add(NewQueryIndex);
	if(RecentQueries.Num() > NumDrawRecentQueries)
	{
		RecentQueries.RemoveAt(0);
	}

	// Update total queries
	TotalNumQueries++;
}


void SCollisionAnalyzer::RebuildFilteredList()
{
	QueryDetailsWidget->ClearCurrentQuery();

	GroupedQueries.Reset();
	RecentQueries.Empty();
	TotalNumQueries = 0.f;

	// Run over results to find which ones pass (@TODO progress bar?)
	for(int32 QueryIdx=0; QueryIdx < Analyzer->Queries.Num(); QueryIdx++)
	{
		const FCAQuery& Query = Analyzer->Queries[QueryIdx];
		if(ShouldDisplayQuery(Query))
		{
			// Passed filter so add to filtered results (defer sorting until end)
			AddQueryToGroupedQueries(QueryIdx, false);
		}
	}

	// We have built all the lists, now sort (if desired)
	if(SortBy == EQuerySortMode::ByTime)
	{
		// Ungrouped
		if(GroupBy == EQueryGroupMode::Ungrouped)
		{
			GroupedQueries.Sort( FCompareQueryByCPUTime(Analyzer) );
		}
		// Grouped
		else
		{
			GroupedQueries.Sort( FCompareGroupByCPUTime() );
			for(int i=0; i<GroupedQueries.Num(); i++)
			{
				TSharedPtr<FQueryTreeItem> Group = GroupedQueries[i];
				check(Group->bIsGroup);
				Group->QueriesInGroup.Sort( FCompareQueryByCPUTime(Analyzer) );
			}
		}
	}
	else if (SortBy == EQuerySortMode::ByID)
	{
		// Ungrouped
		if (GroupBy == EQueryGroupMode::Ungrouped)
		{
			GroupedQueries.Sort(FCompareQueryByID(Analyzer, SortDirection));
		}
		// Grouped
		else
		{
			GroupedQueries.Sort(FCompareGroupByCPUTime()); // Sort groups by time again, doesn't make sense to sort by 
			for (int i = 0; i < GroupedQueries.Num(); i++)
			{
				TSharedPtr<FQueryTreeItem> Group = GroupedQueries[i];
				check(Group->bIsGroup);
				Group->QueriesInGroup.Sort(FCompareQueryByID(Analyzer, SortDirection));
			}
		}
	}
	
	// When underlying array changes, refresh list
	QueryTreeWidget->RequestTreeRefresh();
}

//////////////////////////////////////////////////////////////////////////

const FSlateBrush* SCollisionAnalyzer::GetRecordButtonBrush() const
{
	if(Analyzer->IsRecording())
	{
		// If recording, show stop button
		return FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.Stop");
	}
	else
	{
		// If stopped, show record button
		return FCollisionAnalyzerStyle::Get()->GetBrush("CollisionAnalyzer.Record");
	}
}

FText SCollisionAnalyzer::GetStatusText() const
{
	return FText::Format(
		LOCTEXT("CollisionAnalyzerStatusTextFmt", "Total: {0} queries over {1} frames. Shown: {2} queries"), 
		FText::AsNumber(Analyzer->Queries.Num()), FText::AsNumber(Analyzer->GetNumFramesOfRecording()), FText::AsNumber(TotalNumQueries)
		);
}

ECheckBoxState SCollisionAnalyzer::GetDrawRecentState() const
{
	return bDrawRecentQueries ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


FReply SCollisionAnalyzer::OnRecordButtonClicked()
{
	// Toggle recording state
	Analyzer->SetIsRecording(!Analyzer->IsRecording());

	return FReply::Handled();
}

FReply SCollisionAnalyzer::OnLoadButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		// Default path to find stats
		const FString DefaultPath = FPaths::ProfilingDir() + TEXT("CollisionAnalyzer");

		// File open dialog
		TArray<FString> Filenames;
		bool bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("CollisionFileOpen", "Choose collision file to load").ToString(),
			DefaultPath,
			TEXT(""),
			TEXT( "UCA file|*.uca" ),
			EFileDialogFlags::None,
			Filenames
			);

		// If we chose a file
		if(bOpened && Filenames.Num() > 0)
		{
			Analyzer->LoadCollisionProfileData(Filenames[0]);
		}
	}
	return FReply::Handled();
}

FReply SCollisionAnalyzer::OnSaveButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		// Default path to find stats
		const FString DefaultPath = FPaths::ProfilingDir() + TEXT("CollisionAnalyzer");

		// File save dialog
		TArray<FString> Filenames;
		bool bSaved = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("CollisionFileLocation", "Choose file location").ToString(),
			DefaultPath,
			TEXT(""),
			TEXT( "UCA file|*.uca" ),
			EFileDialogFlags::None,
			Filenames
			);

		// If we chose a file
		if(bSaved && Filenames.Num() > 0)
		{
			Analyzer->SaveCollisionProfileData(Filenames[0]);
		}
	}

	return FReply::Handled();
}

void SCollisionAnalyzer::OnDrawRecentChanged(ECheckBoxState NewState)
{
	bDrawRecentQueries = (NewState == ECheckBoxState::Checked);
}

// By frame

ECheckBoxState SCollisionAnalyzer::GetGroupByFrameState() const
{
	return (GroupBy == EQueryGroupMode::ByFrameNum) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCollisionAnalyzer::OnGroupByFrameChanged(ECheckBoxState NewState)
{
	GroupBy = (NewState == ECheckBoxState::Checked) ? EQueryGroupMode::ByFrameNum : EQueryGroupMode::Ungrouped;
	RebuildFilteredList();
}


// By Tag

ECheckBoxState SCollisionAnalyzer::GetGroupByTagState() const
{
	return (GroupBy == EQueryGroupMode::ByTag) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCollisionAnalyzer::OnGroupByTagChanged(ECheckBoxState NewState)
{
	GroupBy = (NewState == ECheckBoxState::Checked) ? EQueryGroupMode::ByTag : EQueryGroupMode::Ungrouped;
	RebuildFilteredList();
}

// By Owner

ECheckBoxState SCollisionAnalyzer::GetGroupByOwnerState() const
{
	return (GroupBy == EQueryGroupMode::ByOwnerTag) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCollisionAnalyzer::OnGroupByOwnerChanged(ECheckBoxState NewState)
{
	GroupBy = (NewState == ECheckBoxState::Checked) ? EQueryGroupMode::ByOwnerTag : EQueryGroupMode::Ungrouped;
	RebuildFilteredList();
}

void SCollisionAnalyzer::FilterTextCommitted(const FText& CommentText, ETextCommit::Type CommitInfo)
{
	UpdateFilterInfo();
	RebuildFilteredList();
}

void SCollisionAnalyzer::OnSortByChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnName, const EColumnSortMode::Type NewSortMode)
{
	if(ColumnName == TimeColumnName)
	{
		// Only makes sense to sort by time decreasing
		SortBy = EQuerySortMode::ByTime;
		SortDirection = EColumnSortMode::Descending;
	}
	else if(ColumnName == IDColumnName)
	{
		// If already sorting by ID, flip direction
		if (SortBy == EQuerySortMode::ByID)
		{
			SortDirection = (SortDirection == EColumnSortMode::Descending) ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
		}
		// If not, sort by ID, and default to ascending
		else
		{
			SortBy = EQuerySortMode::ByID;
			SortDirection = EColumnSortMode::Descending;
		}
	}

	RebuildFilteredList();
}

EColumnSortMode::Type SCollisionAnalyzer::GetIDSortMode() const
{
	return (SortBy == EQuerySortMode::ByID) ? SortDirection : EColumnSortMode::None;
}

EColumnSortMode::Type SCollisionAnalyzer::GetTimeSortMode() const
{
	return (SortBy == EQuerySortMode::ByTime) ? SortDirection : EColumnSortMode::None;
}

FString SCollisionAnalyzer::QueryTypeToString(ECAQueryType::Type QueryType)
{
	switch (QueryType)
	{
	case ECAQueryType::Raycast:
		return FString(TEXT("Raycast"));
	case ECAQueryType::GeomSweep:
		return FString(TEXT("Sweep"));
	case ECAQueryType::GeomOverlap:
		return FString(TEXT("Overlap"));
	}

	return FString(TEXT("UNKNOWNN"));
}

FString SCollisionAnalyzer::QueryShapeToString(ECAQueryShape::Type QueryShape)
{
	switch(QueryShape)
	{
	case ECAQueryShape::Sphere:
		return FString(TEXT("Sphere"));
	case ECAQueryShape::Box:
		return FString(TEXT("Box"));
	case ECAQueryShape::Capsule:
		return FString(TEXT("Capsule"));	
	case ECAQueryShape::Convex:
		return FString(TEXT("Convex"));
	}

	return FString(TEXT("UNKNOWNN"));
}


FString SCollisionAnalyzer::QueryModeToString(ECAQueryMode::Type QueryMode)
{
	switch(QueryMode)
	{
	case ECAQueryMode::Test:
		return FString(TEXT("Test"));
	case ECAQueryMode::Single:
		return FString(TEXT("Single"));
	case ECAQueryMode::Multi:
		return FString(TEXT("Multi"));
	}

	return FString(TEXT("UNKNOWNN"));
}

#undef LOCTEXT_NAMESPACE
