// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "CollisionAnalyzer.h"

class SCAQueryDetails;
class SEditableTextBox;

class FQueryTreeItem
{
public:
	// Bool to indicate whether this is a group or a leaf individual query entry
	bool								bIsGroup;

	// If a group
	FName								GroupName;
	int32								FrameNum;
	float								TotalCPUTime;
	TArray<TSharedPtr<FQueryTreeItem> >	QueriesInGroup;

	// If a leaf (single query entry)
	int32								QueryIndex;

	static TSharedRef<FQueryTreeItem> MakeGroup(FName InGroupName, int32 InFrameNum)
	{
		TSharedRef<FQueryTreeItem> NewItem = MakeShareable(new FQueryTreeItem(true, InGroupName, InFrameNum, 0.f, INDEX_NONE));
		return NewItem;
	}

	static TSharedRef<FQueryTreeItem> MakeQuery(int32 InQueryIndex)
	{
		TSharedRef<FQueryTreeItem> NewItem = MakeShareable(new FQueryTreeItem(false, NAME_None, INDEX_NONE, 0.f, InQueryIndex));
		return NewItem;
	}

	/** Recalculate the totaly CPU time for this group */
	void UpdateTotalCPUTime(FCollisionAnalyzer* Analyzer)
	{
		if(bIsGroup)
		{
			TotalCPUTime = 0.f;
			// Iterate over each query in group
			for(int32 i=0; i<QueriesInGroup.Num(); i++)
			{
				TSharedPtr<FQueryTreeItem> ChildItem = QueriesInGroup[i];
				check(ChildItem.IsValid());
				check(!ChildItem->bIsGroup);
				check(ChildItem->QueryIndex < Analyzer->Queries.Num());
				const FCAQuery& Query = Analyzer->Queries[ChildItem->QueryIndex];
				TotalCPUTime += Query.CPUTime;
			}

		}
	}

private:
	FQueryTreeItem(bool bInIsGroup, FName InGroupName, int32 InFrameNum, float InTotalCPUTime, int32 InQueryIndex)
		: bIsGroup(bInIsGroup)
		, GroupName(InGroupName)
		, FrameNum(InFrameNum)
		, TotalCPUTime(InTotalCPUTime)
		, QueryIndex(InQueryIndex)
	{}
};

namespace EQueryGroupMode
{
	enum Type
	{
		Ungrouped,
		ByFrameNum,
		ByTag,
		ByOwnerTag
	};
}

namespace EQuerySortMode
{
	enum Type
	{
		ByID,
		ByTime
	};
}

/** Main CollisionAnalyzer UI widget */
class SCollisionAnalyzer : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCollisionAnalyzer) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCollisionAnalyzer* InAnalyzer);

	virtual ~SCollisionAnalyzer();

	// Get delegates
	const FSlateBrush* GetRecordButtonBrush() const;
	FText GetStatusText() const;
	ECheckBoxState GetDrawRecentState() const;
	ECheckBoxState GetGroupByFrameState() const;
	ECheckBoxState GetGroupByTagState() const;
	ECheckBoxState GetGroupByOwnerState() const;
	EColumnSortMode::Type GetIDSortMode() const;
	EColumnSortMode::Type GetTimeSortMode() const;
	// Handler delegates
	FReply OnRecordButtonClicked();
	FReply OnLoadButtonClicked();
	FReply OnSaveButtonClicked();
	void OnDrawRecentChanged(ECheckBoxState NewState);
	void OnGroupByFrameChanged(ECheckBoxState NewState);
	void OnGroupByTagChanged(ECheckBoxState NewState);
	void OnGroupByOwnerChanged(ECheckBoxState NewState);
	void FilterTextCommitted(const FText& CommentText, ETextCommit::Type CommitInfo);
	void OnSortByChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnName, const EColumnSortMode::Type NewSortMode);
	// Table delegates
	TSharedRef<ITableRow> QueryTreeGenerateRow(TSharedPtr<FQueryTreeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnGetChildrenForQueryGroup( TSharedPtr<FQueryTreeItem> InItem, TArray<TSharedPtr<FQueryTreeItem> >& OutChildren );
	void QueryTreeSelectionChanged(TSharedPtr<FQueryTreeItem> SelectedItem, ESelectInfo::Type SelectInfo);

	/** Util to convert a query shape to a string */
	static FString QueryShapeToString(ECAQueryShape::Type QueryShape);

	/** Util to convert a query type to a string */
	static FString QueryTypeToString(ECAQueryType::Type QueryType);

	/** Util to convert a query mode to a string */
	static FString QueryModeToString(ECAQueryMode::Type QueryMode);

	/** Pointer to the analyzer object we want to show ui for */
	FCollisionAnalyzer*				Analyzer;

	/** Current way that we are grouping queries */
	EQueryGroupMode::Type			GroupBy;
	/** Current way we are sorting queries */
	EQuerySortMode::Type			SortBy;
	/** Current way we are setting ID sort direction */
	EColumnSortMode::Type			SortDirection;
private:
	/** Called when the queries array in the CollisionAnalyzer change */
	void OnQueriesChanged();
	/** Called when a single query is added */
	void OnQueryAdded();
	/** Regenerate the VisibleQueries list based on current filter */
	void RebuildFilteredList();
	/** Add a Query to  */
	void AddQueryToGroupedQueries(int32 NewQueryIndex, bool bPerformSort);
	/** Find a group by name or framenum */
	TSharedPtr<FQueryTreeItem> FindQueryGroup(FName InGroupName, int32 InFrameNum);
	/** Update list of queries to draw in 3D */
	void UpdateDrawnQueries();
	/** Update filtering members from entry widgets */
	void UpdateFilterInfo();

	/** See if a particular query passes the current filter */
	bool ShouldDisplayQuery(const FCAQuery& Query);

	// MEMBERS


	/** Index into Analyzer->Queries array for entries you want to show. */
	TArray<TSharedPtr<FQueryTreeItem> >	GroupedQueries;
	/** Total number of queries in analyzer database */
	int32 TotalNumQueries;
	/** Set of of most recent queries */
	TArray<int32> RecentQueries;

	/** If we should draw new queries that pass the filter right away */
	bool bDrawRecentQueries;

	int32 FrameFilterNum;
	FString TagFilterString;
	FString OwnerFilterString;
	float MinCPUFilterTime;

	// WIDGETS

	/** Main query list widget */
	TSharedPtr<STreeView<TSharedPtr<FQueryTreeItem> > >	QueryTreeWidget;
	/** Widget for display details on a specific query */
	TSharedPtr<SCAQueryDetails>							QueryDetailsWidget;
	/** Box to filter to a specific frame */
	TSharedPtr<SEditableTextBox>	FrameFilterBox;
	/** Box to filter to a specific tag */
	TSharedPtr<SEditableTextBox>	TagFilterBox;
	/** Box to filter to a specific owner */
	TSharedPtr<SEditableTextBox>	OwnerFilterBox;
	/** Box to filter by time */
	TSharedPtr<SEditableTextBox>	TimeFilterBox;

	// Column names
public:
	static const FName IDColumnName;
	static const FName FrameColumnName;
	static const FName TypeColumnName;
	static const FName ShapeColumnName;
	static const FName ModeColumnName;
	static const FName TagColumnName;
	static const FName OwnerColumnName;
	static const FName NumBlockColumnName;
	static const FName NumTouchColumnName;
	static const FName TimeColumnName;

};
