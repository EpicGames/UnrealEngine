// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FSpawnTabArgs;
class ICollisionAnalyzer;

class FCollisionAnalyzerModule : public IModuleInterface
{
public:
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface

	/** Gets the debugger singleton or returns NULL */
	static ICollisionAnalyzer* Get()
	{
		static FName CollisionAnalyzerModuleName("CollisionAnalyzer");
		FCollisionAnalyzerModule& DebuggerModule = FModuleManager::Get().LoadModuleChecked<FCollisionAnalyzerModule>(CollisionAnalyzerModuleName);
		return DebuggerModule.GetSingleton();
	}

private:
	virtual ICollisionAnalyzer* GetSingleton() const 
	{ 
		return CollisionAnalyzer; 
	}

	/** Spawns the Collision Analyzer tab in an SDockTab */
	TSharedRef<class SDockTab> SpawnCollisionAnalyzerTab(const FSpawnTabArgs& Args);

	ICollisionAnalyzer* CollisionAnalyzer;
};
