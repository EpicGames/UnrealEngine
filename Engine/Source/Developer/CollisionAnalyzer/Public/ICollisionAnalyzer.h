// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"

class SWidget;
struct FCollisionObjectQueryParams;
struct FCollisionResponseParams;
struct FHitResult;

/** Enum to describe type of the query */
namespace ECAQueryType
{
	enum Type
	{
		Raycast,
		GeomSweep,
		GeomOverlap
	};
}

/** Enum to describe shape of the query */
namespace ECAQueryShape
{
	enum Type
	{
		Sphere,
		Box,
		Capsule,
		Convex
	};
}

/** Enum to describe the mode of query performed */
namespace ECAQueryMode
{
	enum Type
	{
		Test,
		Single,
		Multi
	};
}

class ICollisionAnalyzer
{
public:
	/** Virtual destructor */
	virtual ~ICollisionAnalyzer() = default;

	/** */
	virtual void CaptureQuery(
		const FVector& Start, 
		const FVector& End, 
		const FQuat& Rot, 
		ECAQueryType::Type QueryType, 
		ECAQueryShape::Type QueryShape,
		ECAQueryMode::Type QueryMode,
		const FVector& Dims, 
		ECollisionChannel TraceChannel, 
		const struct FCollisionQueryParams& Params, 
		const FCollisionResponseParams&	ResponseParams,
		const FCollisionObjectQueryParams&	ObjectParams,
		const TArray<FHitResult>& Results, 
		const TArray<FHitResult>& TouchAllResults,
		double CPUTime) = 0;

	/** Returns a new Collision Analyzer widget. */
	virtual TSharedPtr<SWidget> SummonUI() = 0;

	/** */
	virtual void TickAnalyzer(UWorld* InWorld) = 0;

	/** */
	virtual bool IsRecording() = 0;
};
